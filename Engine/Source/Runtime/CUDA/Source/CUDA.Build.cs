// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;
using System.Globalization;

public class CUDA : ModuleRules
{
	public CUDA(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange([
			"Core",
			"RenderCore",
			"RHI",
			"Engine",
		]);
		
		PublicDependencyModuleNames.Add("CUDAHeader");

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PublicDefinitions.Add("PLATFORM_SUPPORTS_CUDA=1");

			PrivateIncludePathModuleNames.Add("VulkanRHI");

			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
			{
				PrivateIncludePathModuleNames.AddRange(["D3D11RHI", "D3D12RHI"]);
			}
		}
		else
		{
			PublicDefinitions.Add("PLATFORM_SUPPORTS_CUDA=0");
		}
	}
}
