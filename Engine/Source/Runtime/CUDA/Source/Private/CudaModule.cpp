// Copyright Epic Games, Inc. All Rights Reserved.
#include "CudaModule.h"

#include "Misc/CoreDelegates.h"

#include "RHI.h"

#if PLATFORM_SUPPORTS_CUDA
	#include "IVulkanDynamicRHI.h"
#endif

#if PLATFORM_WINDOWS
	#include "DynamicRHI.h"
	#include "Windows/AllowWindowsPlatformTypes.h"
	#include "dxgi.h"
	#include "d3d12.h"
	#include "d3d11.h"
	#include "Windows/HideWindowsPlatformTypes.h"

	#include "ID3D11DynamicRHI.h"
	#include "ID3D12DynamicRHI.h"
#endif

#include "HAL/UnrealMemory.h"

#include "Engine/Engine.h"

DEFINE_LOG_CATEGORY(LogCUDA);

// CUDA 11.0.0 is our minimum required version
#define CUDA_MINIMUM_REQUIRED_VERSION 11000

void FCUDAModule::StartupModule()
{
	// Attempt to load the CUDA library, if library loading was successful:
	// When the plugin loading happens in the engine initialization, wire up our post-init delegate.
	// If the plugin loading is on demand, calls InitCuda directly.
	if (LoadCuda() == true)
	{
		if (!GEngine || !GEngine->IsInitialized())
		{
			PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCUDAModule::InitCuda);

			UE_LOGF(LogCUDA, Display, "CUDA module ready pending PostEngineInit.");
		}
		else
		{
			FCUDAModule::InitCuda();
			UE_LOGF(LogCUDA, Display, "CUDA module loading directly as PostEngineInit has passed.");
		}
	}
}

void FCUDAModule::ShutdownModule()
{
	UnloadCuda();

	if (PostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
		PostEngineInitHandle.Reset();
	}
}

bool FCUDAModule::IsAvailable()
{
	return (DriverLibrary != nullptr);
}

const CUDA_DRIVER_API_FUNCTION_LIST* FCUDAModule::DriverAPI()
{
	return &DriverApiPtrs;
}

bool FCUDAModule::LoadCuda()
{
	// Ensure we do not load the shared library for the CUDA Driver API twice
	UnloadCuda();

	// Attempt to load the shared library for the CUDA Driver API
	DriverLibrary = OpenCudaDriverLibrary();
	if (DriverLibrary)
	{
		// Attempt to retrieve the list of function pointers for the Driver API
		LoadCudaDriverFunctions(DriverLibrary, &DriverApiPtrs);

		// Verify that we were able to load the function pointers and that the minimum CUDA version requirement is met
		int driverVersion = -1;
		if (DriverApiPtrs.cuDriverGetVersion != nullptr)
		{
			if (DriverApiPtrs.cuDriverGetVersion(&driverVersion) == CUDA_SUCCESS)
			{
				if (driverVersion >= CUDA_MINIMUM_REQUIRED_VERSION)
				{
					return true;
				}
			}
		}

		// If we reached this point then loading failed
		UnloadCuda();
	}

	return false;
}

void FCUDAModule::UnloadCuda()
{
	// Close shared library for the CUDA Driver API if it is currently loaded
	if (DriverLibrary != nullptr)
	{
		CloseCudaLibrary(DriverLibrary);
		DriverLibrary = nullptr;
	}

	// Zero-out our list of function pointers
	FMemory::Memset(&DriverApiPtrs, 0, sizeof(CUDA_DRIVER_API_FUNCTION_LIST));
}

bool FCUDAModule::IsDeviceIndexRHISelected(int DeviceIndex)
{
#ifdef _MSC_VER
#	pragma warning(push)
#	pragma warning(disable:6398) // The address-of a field cannot be null in well-defined code
#endif
	if (DriverLibrary == nullptr || &DriverApiPtrs == nullptr)
	{
		UE_LOGF(LogCUDA, Warning, "Failed to check IsRHISelectedDevice at %ls:%u", ANSI_TO_TCHAR(__FILE__), __LINE__);
		return false;
	}
#ifdef _MSC_VER
#	pragma warning(pop)
#endif

	// Get the current CUDA device.
	CUdevice cuDevice;
	CUresult getDeviceErr = DriverApiPtrs.cuDeviceGet(&cuDevice, DeviceIndex);
	if (getDeviceErr != CUDA_SUCCESS)
	{
		UE_LOGF(LogCUDA, Warning, "Could not get CUDA device at device %d.", DeviceIndex);
		return false;
	}

	return IsRHISelectedDevice(cuDevice);
}

bool FCUDAModule::IsRHISelectedDevice(CUdevice cuDevice)
{
#if PLATFORM_SUPPORTS_CUDA
	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

	// VULKAN
	if (RHIType == ERHIInterfaceType::Vulkan)
	{
		uint8 deviceUUID[16];

		// We find the device that the RHI has selected for us so that later we can create a CUDA context on this device.
		FMemory::Memcpy(deviceUUID, GetIVulkanDynamicRHI()->RHIGetVulkanDeviceUUID(), 16);

		// Get the device UUID so we can compare this with what the RHI selected.
		CUuuid cudaDeviceUUID;
		CUresult getUuidErr = DriverApiPtrs.cuDeviceGetUuid(&cudaDeviceUUID, cuDevice);
		if (getUuidErr != CUDA_SUCCESS)
		{
			UE_LOGF(LogCUDA, Warning, "Could not get CUDA device UUID at device %d.", cuDevice);
			return false;
		}

		// Compare the CUDA device UUID with RHI selected UUID.
		return FMemory::Memcmp(&cudaDeviceUUID, deviceUUID, 16) == 0;
	}
	else
	#if PLATFORM_WINDOWS
		// DX11
		if (RHIType == ERHIInterfaceType::D3D11)
		{
			char* DeviceLuid = new char[64];
			unsigned int DeviceNodeMask;
			CUresult getLuidErr = DriverApiPtrs.cuDeviceGetLuid(DeviceLuid, &DeviceNodeMask, cuDevice);
			if (getLuidErr != CUDA_SUCCESS)
			{
				UE_LOGF(LogCUDA, Warning, "Could not get CUDA device LUID at device %d.", cuDevice);
				return false;
			}

			ID3D11Device* NativeD3D11Device = GetID3D11DynamicRHI()->RHIGetDevice();

			TRefCountPtr<IDXGIDevice> DXGIDevice;

			HRESULT Result = NativeD3D11Device->QueryInterface(__uuidof(IDXGIDevice), (void**)DXGIDevice.GetInitReference());
			if (FAILED(Result))
			{
				UE_LOGF(LogCUDA, Error, "Failed to get DXGIDevice when trying to resolve to CUDA device at %ls:%u \n with error %u", ANSI_TO_TCHAR(__FILE__), __LINE__, Result);
				return false;
			}

			// VERIFYD3D11RESULT_EX(GDynamicRHI->GetDevice()->QueryInterface(IID_IDXGIDevice, (void**)DXGIDevice.GetInitReference()), GDynamicRHI->GetDevice());

			TRefCountPtr<IDXGIAdapter> DXGIAdapter;
			DXGIDevice->GetAdapter((IDXGIAdapter**)DXGIAdapter.GetInitReference());

			DXGI_ADAPTER_DESC AdapterDesc;
			DXGIAdapter->GetDesc(&AdapterDesc);

			LUID AdapterLUID = AdapterDesc.AdapterLuid;

			return ((memcmp(&AdapterLUID.LowPart, DeviceLuid,
						 sizeof(AdapterLUID.LowPart))
						== 0)
				&& (memcmp(&AdapterLUID.HighPart,
						DeviceLuid + sizeof(AdapterLUID.LowPart),
						sizeof(AdapterLUID.HighPart))
					== 0));
		}
		// DX12
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			char* DeviceLuid = new char[64];
			unsigned int DeviceNodeMask;
			CUresult getLuidErr = DriverApiPtrs.cuDeviceGetLuid(DeviceLuid, &DeviceNodeMask, cuDevice);
			if (getLuidErr != CUDA_SUCCESS)
			{
				UE_LOGF(LogCUDA, Warning, "Could not get CUDA device LUID at device %d.", cuDevice);
				return false;
			}

			ID3D12Device* NativeD3D12Device = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();

			LUID AdapterLUID = NativeD3D12Device->GetAdapterLuid();

			return ((memcmp(&AdapterLUID.LowPart, DeviceLuid,
						 sizeof(AdapterLUID.LowPart))
						== 0)
				&& (memcmp(&AdapterLUID.HighPart,
						DeviceLuid + sizeof(AdapterLUID.LowPart),
						sizeof(AdapterLUID.HighPart))
					== 0));
		}
		else if (RHIType == ERHIInterfaceType::OpenGL)
		{
			UE_LOGF(LogCUDA, Warning, "CUDA is unsupported on OpenGL.");
			return false;
		}
		else
	#endif // PLATFORM_WINDOWS
#endif	   // PLATFORM_SUPPORTS_CUDA
		{
			UE_LOGF(LogCUDA, Warning, "Unsupported RHI or Platform for CUDA.");
		}

	return false;
}

void FCUDAModule::InitCuda()
{
	// Remove the binding
	if (PostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
		PostEngineInitHandle.Reset();
	}

	// Unload CUDA if we are not running the RHI on an NVIDIA device this can happen if the NVIDIA drivers
	// are present on a system with an AMD GPU
	// TODO is there the potential case that someone will want to use CUDA on a 2nd GPU
	// when running the rendering on an AMD one?
	if (!IsRHIDeviceNVIDIA())
	{
		UE_LOGF(LogCUDA, Display, "RHI device not NVIDIA unloading CUDA support");
		UnloadCuda();
		return;
	}

	// Initialise to rhiDeviceIndex -1, which is an invalid device index, if it remains -1 we will know no device was found.
	rhiDeviceIndex = -1;

#if PLATFORM_SUPPORTS_CUDA
	// Initialise CUDA API
	{
		UE_LOGF(LogCUDA, Display, "Initialising CUDA API...");

		CUresult err = DriverApiPtrs.cuInit(0);
		if (err == CUDA_SUCCESS)
		{
			UE_LOGF(LogCUDA, Display, "CUDA API initialised successfully.");
		}
		else
		{
			UE_LOGF(LogCUDA, Error, "CUDA API failed to initialise.");
			UnloadCuda();
			return;
		}
	}

	// Find out how many graphics devices there are so we find that one that matches the one the RHI selected.
	int device_count = 0;
	CUresult deviceCountErr = DriverApiPtrs.cuDeviceGetCount(&device_count);
	if (deviceCountErr == CUDA_SUCCESS)
	{
		UE_LOGF(LogCUDA, Display, "Found %d CUDA capable devices.", device_count);
	}
	else
	{
		UE_LOGF(LogCUDA, Error, "Could not count how many graphics devices there are using CUDA.");
	}

	if (device_count == 0)
	{
		UE_LOGF(LogCUDA, Error, "There are no available device(s) that support CUDA. If that is untrue check CUDA is installed.");
		return;
	}

	CUdevice cuDevice;
	CUcontext cudaContext;

	// Find the GPU device that is selected by the RHI.
	for (int current_device = 0; current_device < device_count; ++current_device)
	{
		// Get the current CUDA device.
		CUresult getDeviceErr = DriverApiPtrs.cuDeviceGet(&cuDevice, current_device);
		if (getDeviceErr != CUDA_SUCCESS)
		{
			UE_LOGF(LogCUDA, Warning, "Could not get CUDA device at device %d.", current_device);
			continue;
		}

		// Get device name for logging purposes.
		char deviceNameArr[256];
		CUresult getNameErr = DriverApiPtrs.cuDeviceGetName(deviceNameArr, 256, cuDevice);
		FString deviceName;
		if (getNameErr == CUDA_SUCCESS)
		{
			deviceName = FString(UTF8_TO_TCHAR(deviceNameArr));
			UE_LOGF(LogCUDA, Display, "Found device %d called \"%ls\".", current_device, *deviceName);
		}
		else
		{
			UE_LOGF(LogCUDA, Warning, "Could not get name of CUDA device %d.", current_device);
		}

		// Compare the CUDA device with current RHI selected device.
		if (this->IsRHISelectedDevice(cuDevice))
		{
			UE_LOGF(LogCUDA, Display, "Attempting to create CUDA context on GPU Device %d...", current_device);

			CUresult createCtxErr = DriverApiPtrs.cuCtxCreate(&cudaContext, 0, current_device);
			if (createCtxErr == CUDA_SUCCESS)
			{
				UE_LOGF(LogCUDA, Display, "Created CUDA context on device \"%ls\".", *deviceName);
				contextMap.Add(current_device, cudaContext);
				rhiDeviceIndex = current_device;
				break;
			}
			else
			{
				UE_LOGF(LogCUDA, Error, "CUDA module could not create CUDA context on RHI device \"%ls\".", *deviceName);
				return;
			}
		}
		else
		{
			UE_LOGF(LogCUDA, Verbose, "CUDA device %d \"%ls\" does not match RHI device.", current_device, *deviceName);
		}
	}

	if (rhiDeviceIndex == -1)
	{
		UE_LOGF(LogCUDA, Error, "CUDA module failed to create a CUDA context on the RHI selected device.");
		return;
	}
	else
	{
		OnPostCUDAInit.Broadcast();
	}
#else

	UE_LOGF(LogCUDA, Error, "CUDA attemped to be initialized on unsupported Platform (Currently supported platforms are Win32, Win64, Linux and LinuxAArch64).");

#endif // PLATFORM_SUPPORTS_CUDA
}

CUcontext FCUDAModule::GetCudaContext()
{
	if (IsAvailable() == false)
	{
		UE_LOGF(LogCUDA, Error, "You have requested a CUDA context when the CUDA Driver API is not loaded.");
		return nullptr;
	}

	if (rhiDeviceIndex == -1)
	{
		UE_LOGF(LogCUDA, Error, "You have requested a CUDA context when the RHI selected device does not have a CUDA context, did initialisation fail?");
		return nullptr;
	}

	return contextMap[rhiDeviceIndex];
}

uint32 FCUDAModule::GetCudaDeviceIndex() const
{
	return rhiDeviceIndex;
}

CUcontext FCUDAModule::GetCudaContextForDevice(int DeviceIndex)
{
	if (IsAvailable() == false)
	{
		UE_LOGF(LogCUDA, Fatal, "You have requested a CUDA context when the CUDA Driver API is not loaded.");
	}

	if (!contextMap.Contains(DeviceIndex))
	{
		CUcontext cudaContext = nullptr;
		CUresult createCtxErr = DriverApiPtrs.cuCtxCreate(&cudaContext, 0, DeviceIndex);
		if (createCtxErr == CUDA_SUCCESS)
		{
			UE_LOGF(LogCUDA, Display, "Created a new CUDA context on device %d on request.", DeviceIndex);
			contextMap.Add(DeviceIndex, cudaContext);
		}
		else
		{
			UE_LOGF(LogCUDA, Error, "Failed to create a CUDA context on device %d on request.", DeviceIndex);
			return cudaContext;
		}
	}

	return contextMap[DeviceIndex];
}

IMPLEMENT_MODULE(FCUDAModule, CUDA);
