// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CudaWrapper.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
DECLARE_MULTICAST_DELEGATE(FOnPostCUDAInit)

DECLARE_LOG_CATEGORY_EXTERN(LogCUDA, Log, All);
class FCUDAModule : public IModuleInterface
{
public:
	static CUDA_DRIVER_API_FUNCTION_LIST CUDA()
	{
		return FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").DriverApiPtrs;
	}

	CUDA_API virtual void StartupModule() override;
	CUDA_API virtual void ShutdownModule() override;

	// Determines if the CUDA Driver API is available for use
	CUDA_API bool IsAvailable();

	// Retrieves the function pointer list for the CUDA Driver API
	CUDA_API const CUDA_DRIVER_API_FUNCTION_LIST* DriverAPI();

	// Retrieves the CUDA context for the GPU device currently in use by the Vulkan RHI
	CUDA_API CUcontext GetCudaContext();
	
	// Retrieves the device index for the current context
	CUDA_API uint32 GetCudaDeviceIndex() const;
	
	// Retrieves or creates the CUDA context for the specified GPU device
	CUDA_API CUcontext GetCudaContextForDevice(int DeviceIndex);

	// Called after CUDA is loaded successfully
	FOnPostCUDAInit OnPostCUDAInit;

	CUDA_API bool IsDeviceIndexRHISelected(int DeviceIndex);

private:
	CUDA_API bool LoadCuda();
	CUDA_API void UnloadCuda();

	CUDA_API void InitCuda();
	CUDA_API bool IsRHISelectedDevice(CUdevice cuDevice);

	void* DriverLibrary;
	CUDA_DRIVER_API_FUNCTION_LIST DriverApiPtrs;

	uint32 rhiDeviceIndex;
	TMap<uint32, CUcontext> contextMap;

	FDelegateHandle PostEngineInitHandle;
};
